{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Avid DNxHD® decoder for Electra",
	"Description": "Avid DNxHD® video decoder",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Category": "Media Players",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "AvidDNxHDDecoderElectra",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "ElectraCodecs",
			"Enabled": true
		}
	]
}
