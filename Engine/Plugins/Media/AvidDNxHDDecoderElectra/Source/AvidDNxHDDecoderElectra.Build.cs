// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AvidDNxHDDecoderElectra : ModuleRules
	{
		public AvidDNxHDDecoderElectra(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
                    "Projects",
					"ElectraBase",
                    "ElectraSamples",
					"ElectraCodecFactory",
					"ElectraDecoders"
                });

            if (Target.Platform == UnrealTargetPlatform.Win64)
            {
				PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"DNxHR",
						//"DNxUncompressed",
					}
				);

				PublicDependencyModuleNames.Add("DirectX");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");

				if (Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
				{
					PublicAdditionalLibraries.AddRange(new string[] {
						Path.Combine(Target.WindowsPlatform.DirectXLibDir, "dxerr.lib"),
					});
				}
			}
		}
	}
}
