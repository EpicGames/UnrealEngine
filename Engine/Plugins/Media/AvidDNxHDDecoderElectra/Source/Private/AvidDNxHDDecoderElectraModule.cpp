// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvidDNxHDDecoderElectraModule.h"
#include "Modules/ModuleManager.h"

#if PLATFORM_WINDOWS
#include "Windows/ElectraMediaAvidDNxHDDecoder.h"
#endif

#define LOCTEXT_NAMESPACE "AvidDNxHDDecoderElectraModule"

DEFINE_LOG_CATEGORY(LogAvidDNxHDElectraDecoder);

class FAvidDNxHDElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
#if PLATFORM_WINDOWS
		FElectraMediaAvidDNxHDDecoder::Startup();
#endif		
	}

	void ShutdownModule() override
	{
#if PLATFORM_WINDOWS
		FElectraMediaAvidDNxHDDecoder::Shutdown();
#endif		
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(FAvidDNxHDElectraDecoderModule, AvidDNxHDDecoderElectra);

#undef LOCTEXT_NAMESPACE
