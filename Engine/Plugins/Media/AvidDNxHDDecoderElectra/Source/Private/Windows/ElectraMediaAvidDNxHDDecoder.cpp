// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaAvidDNxHDDecoder.h"
#include "AvidDNxHDDecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "Modules/ModuleManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"

#include "ElectraDecodersPlatformResources.h"
#include COMPILED_PLATFORM_HEADER(ElectraTextureSampleGPUBufferHelper.h)

#include "AvidDNxCodec.h"
//#include "dnx_uncompressed_sdk.h"

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				2

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FVideoDecoderAvidDNxHDElectra;


class FVideoDecoderOutputAvidDNxHDElectra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputAvidDNxHDElectra()
	{
	}

	FTimespan GetPTS() const override
	{
		return PTS;
	}
	uint64 GetUserValue() const override
	{
		return UserValue;
	}

	EOutputType GetOutputType() const
	{
		return EOutputType::Output;
	}

	int32 GetWidth() const override
	{
		return ImageWidth;
	}
	int32 GetHeight() const override
	{
		return ImageHeight;
	}
	int32 GetDecodedWidth() const override
	{
		return Width;
	}
	int32 GetDecodedHeight() const override
	{
		return Height;
	}
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{
		return Crop;
	}
	int32 GetAspectRatioW() const override
	{
		return AspectW;
	}
	int32 GetAspectRatioH() const override
	{
		return AspectH;
	}
	int32 GetFrameRateNumerator() const override
	{
		return FrameRateN;
	}
	int32 GetFrameRateDenominator() const override
	{
		return FrameRateD;
	}
	int32 GetNumberOfBits() const override
	{
		return NumBits;
	}
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{
		OutExtraValues = ExtraValues;
	}
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputAvidDNxHDElectra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{
		return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported;
	}

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{
		return Codec4CC;
	}
	int32 GetNumberOfBuffers() const override
	{
		return 1;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffer;
		}
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			return GPUBuffer.Resource.GetReference();
		}
#endif
		return nullptr;
	}
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			SyncObject = { GPUBuffer.Fence.GetReference(), GPUBuffer.FenceValue, nullptr };
			return true;
		}
#endif
	return false;
	}
	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return BufferFormat;
		}
		return EPixelFormat::PF_Unknown;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return BufferEncoding;
		}
		return EElectraTextureSamplePixelEncoding::Native;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Pitch;
		}
		return 0;
	}

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 ImageWidth = 0;
	int32 ImageHeight = 0;
	int32 Width = 0;
	int32 Height = 0;
	int32 Pitch = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	int32 PixelFormat = 0;
	TMap<FString, FVariant> ExtraValues;

	uint32 Codec4CC = 0;
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> Buffer;
	EPixelFormat BufferFormat = EPixelFormat::PF_Unknown;
	EElectraTextureSamplePixelEncoding BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	FElectraMediaDecoderOutputBufferPool_DX12::FOutputData GPUBuffer;
#endif
};



class FVideoDecoderAvidDNxHDElectra : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)5));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(true));
	}

	FVideoDecoderAvidDNxHDElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FVideoDecoderAvidDNxHDElectra();

	IElectraDecoder::EType GetType() const override
	{
		return IElectraDecoder::EType::Video;
	}

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	int32 DisplayWidth = 0;
	int32 DisplayHeight = 0;
	int32 DecodedWidth = 0;
	int32 DecodedHeight = 0;
	int32 AspectW = 0;
	int32 AspectH = 0;

	struct FDecoderHandle
	{
		int32 PrepareForDecoding(const void* InInput, uint32 InInputSize, int32 InNumDecodeThreads, bool bIgnoreAlpha);
		int32 GetNumOutputBits() const
		{ return NumOutputBits; }
		int32 Decode(uint8* OutBufferData, uint32 OutBufferSize, const void* InInput, uint32 InInputSize);
		void Reset();
		bool SetupDecompressStruct();
		static int32 GetNumberOfOutputColorBitsPerPixel(DNX_ComponentType_t InOutputComponentType);
		DNX_Decoder Handle = nullptr;
		DNX_CompressedParams_t CurrentCompressedParams;
		DNX_UncompressedParams_t CurrentUncompressedParams;
		DNX_SignalStandard_t CurrentSignalStandard = DNX_SignalStandard_t::DNX_SS_INVALID;
		DNX_DecodeOperationParams_t DecodeOperationParams;
		int32 NumOutputBits = 0;
		bool bHasAlpha = false;
	};
	int32 DesiredNumberOfDecoderThreads = 0;
	FDecoderHandle Decoder;

	IElectraDecoder::FError LastError;

	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	TSharedPtr<FVideoDecoderOutputAvidDNxHDElectra, ESPMode::ThreadSafe> CurrentOutput;
	bool bFlushPending = false;

	uint32 MaxOutputBuffers;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	mutable TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> D3D12ResourcePool;
#endif
};


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FAvidDNxHDVideoDecoderElectraFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FAvidDNxHDVideoDecoderElectraFactory, ESPMode::ThreadSafe>
{
public:
	static constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	virtual ~FAvidDNxHDVideoDecoderElectraFactory()
	{ }

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FVideoDecoderAvidDNxHDElectra::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		switch(InCodecFormat.FourCC)
		{
			case Make4CC('A','V','d','h'):
			{
				if (InCodecFormat.HumanReadableFormatInfo.IsEmpty())
				{
					OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, FVariant(FString(TEXT("Avid DNxHD"))));
				}
				else
				{
					OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, InCodecFormat.HumanReadableFormatInfo);
				}
				OutFormatInfo.Emplace(IElectraDecoderFormatInfo::IsEveryFrameKeyframe, FVariant(true));
				return 1;
			}
			default:
			{
				break;
			}
		}
		return 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? MakeShared<FVideoDecoderAvidDNxHDElectra, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("DNxHR")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString(TEXT("Avid"))); }

	static TSharedPtr<FAvidDNxHDVideoDecoderElectraFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FAvidDNxHDVideoDecoderElectraFactory, ESPMode::ThreadSafe> FAvidDNxHDVideoDecoderElectraFactory::Self;

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

namespace FElectraMediaAvidDNxHDDecoder_Windows
{
	static void* LibHandleDNx = nullptr;
	static void* LibHandleDNxUncompressed = nullptr;
	static bool bWasRegistered = false;
}

void FElectraMediaAvidDNxHDDecoder::Startup()
{
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Load the DLLs
	FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNx = FPlatformProcess::GetDllHandle(TEXT("DNxHR.dll"));
	if (!FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNx)
	{
		UE_LOGF(LogAvidDNxHDElectraDecoder, Error, "Failed to load required library. Plug-in will not be functional.");
		return;
	}
/*
	FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNxUncompressed = FPlatformProcess::GetDllHandle(TEXT("DNxUncompressedSDK.dll"));
	if (!FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNxUncompressed)
	{
		UE_LOGF(LogAvidDNxHDElectraDecoder, Error, "Failed to load required library. Plug-in will not be functional.");
		return;
	}
*/
	int Result = DNX_Initialize();
	if (Result != DNX_NO_ERROR)
	{
		UE_LOGF(LogAvidDNxHDElectraDecoder, Error, "DNX_Initialize() returned error %d. Plug-in will not be functional.", Result);
		return;
	}

	// Create an instance of the factory, which is also the modular feature.
	check(!FAvidDNxHDVideoDecoderElectraFactory::Self.IsValid());
	FAvidDNxHDVideoDecoderElectraFactory::Self = MakeShared<FAvidDNxHDVideoDecoderElectraFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FAvidDNxHDVideoDecoderElectraFactory::Self.Get());
	FElectraMediaAvidDNxHDDecoder_Windows::bWasRegistered = true;
}

void FElectraMediaAvidDNxHDDecoder::Shutdown()
{
	if (FElectraMediaAvidDNxHDDecoder_Windows::bWasRegistered)
	{
		DNX_Finalize();
		FElectraMediaAvidDNxHDDecoder_Windows::bWasRegistered = false;
		IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FAvidDNxHDVideoDecoderElectraFactory::Self.Get());
		FAvidDNxHDVideoDecoderElectraFactory::Self.Reset();
	}
	// Unload the DLLs
	if (FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNx)
	{
		FPlatformProcess::FreeDllHandle(FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNx);
		FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNx = nullptr;
	}
	if (FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNxUncompressed)
	{
		FPlatformProcess::FreeDllHandle(FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNxUncompressed);
		FElectraMediaAvidDNxHDDecoder_Windows::LibHandleDNxUncompressed = nullptr;
	}
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FVideoDecoderAvidDNxHDElectra::FVideoDecoderAvidDNxHDElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	const Electra::FCodecTypeFormat::FVideo& vid(InitialCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
	AspectW = (int32) vid.AspectRatioW;
	AspectH = (int32) vid.AspectRatioH;

	DisplayWidth = (int32) vid.Width;
	DisplayHeight = (int32) vid.Height;
	DecodedWidth = DisplayWidth;
	DecodedHeight = DisplayHeight;

	// FIXME: How many exactly?
	DesiredNumberOfDecoderThreads = 8;

	MaxOutputBuffers = (uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_output_buffers"), 5);
	MaxOutputBuffers += kElectraDecoderPipelineExtraFrames;
}

FVideoDecoderAvidDNxHDElectra::~FVideoDecoderAvidDNxHDElectra()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

void FVideoDecoderAvidDNxHDElectra::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FVideoDecoderAvidDNxHDElectra::GetError() const
{
	return LastError;
}

bool FVideoDecoderAvidDNxHDElectra::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
#if 0
	void DNX_GetErrorString(int errorCode, char* errorStrPtr /*Pointer to a buffer which receives an error description string. Buffer should be at least 60 characters long.*/);
#endif
	return false;
}

void FVideoDecoderAvidDNxHDElectra::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FVideoDecoderAvidDNxHDElectra::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	return IElectraDecoder::ECSDCompatibility::Compatible;
}

bool FVideoDecoderAvidDNxHDElectra::ResetToCleanStart()
{
	bFlushPending = false;
	CurrentOutput.Reset();
	Decoder.Reset();
	return true;
}

static void CopyData(TRefCountPtr<ID3D12Resource> BufferResource, uint32 BufferPitch, const uint8* TempBuffer, uint32 TempBufferPitch, uint32 Height)
{
	uint8* ResourceAddr = nullptr;
	HRESULT Res = BufferResource->Map(0, nullptr, (void**)&ResourceAddr);
	check(SUCCEEDED(Res));

	FElectraMediaDecoderOutputBufferPool_DX12::CopyWithPitchAdjust(ResourceAddr, BufferPitch, TempBuffer, TempBufferPitch, Height);

	// We decoded into the resource: Unmap the resource memory and signal that it's usable by the GPU
	BufferResource->Unmap(0, nullptr);
}

IElectraDecoder::EDecoderError FVideoDecoderAvidDNxHDElectra::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

#if ELECTRA_MEDIAGPUBUFFER_DX12
	// If we will create a new resource pool or we have still buffers in an existing one, we can proceed, else we'd have no resources to output the data
	if (D3D12ResourcePool.IsValid() && !D3D12ResourcePool->BufferAvailable())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}
#endif

	// Decode data. This immediately produces a new output frame.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		int32 Result;
		Result = Decoder.PrepareForDecoding(InInputAccessUnit.Data, (unsigned int)InInputAccessUnit.DataSize, DesiredNumberOfDecoderThreads, false);
		if (Result)
		{
			PostError(Result, TEXT("PrepareForDecoding() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		void* PlatformDevice = nullptr;
		int32 PlatformDeviceVersion = 0;
		bool bUseGPUBuffers = false;
#if ELECTRA_MEDIAGPUBUFFER_DX12
		FElectraDecodersPlatformResources::GetD3DDeviceAndVersion(&PlatformDevice, &PlatformDeviceVersion);
		bUseGPUBuffers = (PlatformDevice && PlatformDeviceVersion >= 12000);
#endif

		TSharedPtr<FVideoDecoderOutputAvidDNxHDElectra, ESPMode::ThreadSafe> NewOutput = MakeShared<FVideoDecoderOutputAvidDNxHDElectra>();
		NewOutput->PTS = InInputAccessUnit.PTS;
		NewOutput->UserValue = InInputAccessUnit.UserValue;

		NewOutput->Width = DecodedWidth;
		NewOutput->Height = DecodedHeight;
		NewOutput->Pitch = NewOutput->Width;
		NewOutput->Crop.Right = DecodedWidth - DisplayWidth;
		NewOutput->Crop.Bottom = DecodedHeight - DisplayHeight;
		if (AspectW && AspectH)
		{
			NewOutput->AspectW = AspectW;
			NewOutput->AspectH = AspectH;
		}
		NewOutput->Codec4CC = InitialCodecFormat.FourCC;
		NewOutput->NumBits = Decoder.GetNumOutputBits();

		NewOutput->ImageWidth = DecodedWidth - NewOutput->Crop.Left - NewOutput->Crop.Right;
		NewOutput->ImageHeight = DecodedHeight - NewOutput->Crop.Top - NewOutput->Crop.Bottom;

		NewOutput->PixelFormat = (int32) Decoder.CurrentUncompressedParams.compType;

		bool bIsPlanar = false;
		switch(Decoder.CurrentUncompressedParams.compType)
		{
			case DNX_CT_UCHAR:
			{
				switch(Decoder.CurrentUncompressedParams.compOrder)
				{
					case DNX_CCO_YCbYCr_NoA:
					case DNX_CCO_CbYCrY_NoA:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_B8G8R8A8;
						NewOutput->BufferEncoding = (Decoder.CurrentUncompressedParams.compOrder == DNX_CCO_CbYCrY_NoA) ?  EElectraTextureSamplePixelEncoding::CbY0CrY1 : EElectraTextureSamplePixelEncoding::Y0CbY1Cr;
						NewOutput->Width /= 2;
						NewOutput->Pitch *= 2;
						break;
					}
					case DNX_CCO_ARGB_Interleaved:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_A8R8G8B8;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
						NewOutput->Pitch *= 4;
						break;
					}
					case DNX_CCO_RGBA_Interleaved:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_R8G8B8A8;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
						NewOutput->Pitch *= 4;
						break;
					}
					case DNX_CCO_CbYCrA_Interleaved:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_B8G8R8A8;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::YCbCr_Alpha;
						NewOutput->Pitch *= 4;
						break;
					}
					case DNX_CCO_YCbCr_Planar:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_NV12;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
						if (!bUseGPUBuffers)
						{
							NewOutput->Height = (NewOutput->Height * 3) / 2;
						}
						bIsPlanar = true;
						break;
					}
					default:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_Unknown;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
					}
				}
				break;
			}
			case DNX_CT_SHORT:
			case DNX_CT_USHORT_10_6:
			case DNX_CT_USHORT_12_4:
			{
				switch(Decoder.CurrentUncompressedParams.compOrder)
				{
					case DNX_CCO_YCbYCr_NoA:
					case DNX_CCO_CbYCrY_NoA:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_A16B16G16R16;
						NewOutput->BufferEncoding = (Decoder.CurrentUncompressedParams.compOrder == DNX_CCO_CbYCrY_NoA) ? EElectraTextureSamplePixelEncoding::CbY0CrY1 : EElectraTextureSamplePixelEncoding::Y0CbY1Cr;
						NewOutput->Width /= 2;
						NewOutput->Pitch *= 4;
						break;
					}
					case DNX_CCO_RGBA_Interleaved:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_R16G16B16A16_UNORM;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
						NewOutput->Pitch *= 8;
						break;
					}
					case DNX_CCO_ABGR_Interleaved:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_A16B16G16R16;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
						NewOutput->Pitch *= 8;
						break;
					}
					case DNX_CCO_CbYCrA_Interleaved:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_A16B16G16R16;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::YCbCr_Alpha;
						NewOutput->Pitch *= 8;
						break;
					}
					case DNX_CCO_YCbCr_Planar:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_P010;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
						if (!bUseGPUBuffers)
						{
							NewOutput->Height = (NewOutput->Height * 3) / 2;
						}
						bIsPlanar = true;
						break;
					}
					default:
					{
						NewOutput->BufferFormat = EPixelFormat::PF_Unknown;
						NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
					}
				}
				break;
			}
			case DNX_CT_V210:
			{
				NewOutput->BufferFormat = EPixelFormat::PF_A2B10G10R10;
				NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::CbY0CrY1;
				break;
			}
			default:
			{
				NewOutput->BufferFormat = EPixelFormat::PF_Unknown;
				NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
			}
		}

		NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("avid")));
		NewOutput->ExtraValues.Emplace(TEXT("codec_4cc"), FVariant(InitialCodecFormat.FourCC));

		NewOutput->ExtraValues.Emplace(TEXT("avid_color_volume"), FVariant((int32) Decoder.CurrentUncompressedParams.colorVolume));

		uint32 BPP = NewOutput->Pitch / NewOutput->Width;

		uint8* OutBufferData = nullptr;
		uint32 OutBufferSize = 0;
		TArray<uint8> TempBuffer;

#if ELECTRA_MEDIAGPUBUFFER_DX12
		uint32 ResourcePitch = 0;
		if (bUseGPUBuffers)
		{
			TRefCountPtr D3D12Device(static_cast<ID3D12Device*>(PlatformDevice));
			HRESULT HResult;
			FString HResultMsg;

			uint32 AllocHeight = bIsPlanar ? (NewOutput->Height * 3 / 2) : NewOutput->Height;

			// Create the resource pool as needed...
			if (!D3D12ResourcePool || !D3D12ResourcePool->IsCompatibleAsBuffer(MaxOutputBuffers, NewOutput->Width, AllocHeight, BPP))
			{
				// Note: if we reconfigure the pool, we will end up with >1 heaps until all older users are destroyed - so streams with lots of changes will be quite resource hungry
				D3D12ResourcePool = MakeShared<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>(D3D12Device, MaxOutputBuffers, NewOutput->Width, AllocHeight, BPP);
			}

			// Request resource and fence...
			if (D3D12ResourcePool->AllocateOutputDataAsBuffer(HResult, HResultMsg, NewOutput->GPUBuffer, ResourcePitch))
			{
				NewOutput->Pitch = ResourcePitch;

				uint32 OutputPitch = BPP * NewOutput->Width;

				// Pitch compatible with DX12 resource?
				if (OutputPitch != ResourcePitch)
				{
					// No, decode to temp buffer...
					TempBuffer.AddUninitialized(OutputPitch * AllocHeight);

					OutBufferData = TempBuffer.GetData();
					OutBufferSize = TempBuffer.Num();

					check(OutBufferSize <= AllocHeight * ResourcePitch);
				}
				else
				{
					// Yes, decode directly into the DX12 resource...
					HResult = NewOutput->GPUBuffer.Resource->Map(0, nullptr, (void**)&OutBufferData);
					if (FAILED(HResult))
					{
						PostError((int32)HResult, FString::Printf(TEXT("Failed to map D3D12 buffer")), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
						return IElectraDecoder::EDecoderError::Error;
					}
					OutBufferSize = ResourcePitch * AllocHeight;
				}
			}
			else
			{
				PostError((int32)HResult, FString::Printf(TEXT("Failed to allocate D3D12 buffer (%s)"), *HResultMsg), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		else
#endif
		{
			NewOutput->Buffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
			NewOutput->Buffer->AddUninitialized(BPP * NewOutput->Width * NewOutput->Height);

			OutBufferData = NewOutput->Buffer->GetData();
			OutBufferSize = NewOutput->Buffer->Num();
		}

		Result = Decoder.Decode(OutBufferData, OutBufferSize, InInputAccessUnit.Data, (unsigned int)InInputAccessUnit.DataSize);

#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (bUseGPUBuffers)
		{
			if (Result == 0)
			{
				// If we have a temp buffer, we need to copy the data first...
				if (!TempBuffer.IsEmpty())
				{
					CopyData(NewOutput->GPUBuffer.Resource, ResourcePitch, TempBuffer.GetData(), NewOutput->Width * BPP, NewOutput->Height);
					NewOutput->GPUBuffer.Fence->Signal(NewOutput->GPUBuffer.FenceValue);
				}
				else
				{
					// We decoded into the resource: Unmap the resource memory and signal that it's usable by the GPU
					NewOutput->GPUBuffer.Resource->Unmap(0, nullptr);
					NewOutput->GPUBuffer.Fence->Signal(NewOutput->GPUBuffer.FenceValue);
				}
			}
		}
#endif

		if (Result)
		{
			PostError(Result, TEXT("DNX_DecodeFrame() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		CurrentOutput = MoveTemp(NewOutput);
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderAvidDNxHDElectra::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bFlushPending = true;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderAvidDNxHDElectra::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	ResetToCleanStart();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FVideoDecoderAvidDNxHDElectra::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}
	// Pending flush?
	if (bFlushPending)
	{
		bFlushPending = false;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FVideoDecoderAvidDNxHDElectra::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}




int32 FVideoDecoderAvidDNxHDElectra::FDecoderHandle::PrepareForDecoding(const void* InInput, uint32 InInputSize, int32 InNumDecodeThreads, bool bIgnoreAlpha)
{
	DNX_CompressedParams_t CompressedParams;
	DNX_SignalStandard_t SignalStandard = DNX_SignalStandard_t::DNX_SS_INVALID;
	FMemory::Memzero(CompressedParams);
	CompressedParams.structSize = sizeof(CompressedParams);
	int Result = DNX_GetInfoFromCompressedFrame(InInput, InInputSize, &CompressedParams, &SignalStandard);
	if (Result != DNX_NO_ERROR)
	{
		UE_LOGF(LogAvidDNxHDElectraDecoder, Log, "DNX_GetInfoFromCompressedFrame() failed with %d", Result);
		return Result;
	}
	// If we have a decoder see if the format changed and reconfigure the decoder if so.
	if (Handle)
	{
		bool bChanged = SignalStandard != CurrentSignalStandard || FMemory::Memcmp(&CompressedParams, &CurrentCompressedParams, sizeof(CompressedParams)) != 0;
		if (bChanged)
		{
			CurrentCompressedParams = CompressedParams;
			CurrentSignalStandard = SignalStandard;
			bHasAlpha = !!CurrentCompressedParams.alphaPresence;
			DecodeOperationParams.decodeAlpha = bIgnoreAlpha ? 0 : (bHasAlpha ? 1 : 0);
			if (!SetupDecompressStruct())
			{
				return DNX_INVALID_UNCOMPINFO_ERROR;
			}
			Result = DNX_ConfigureDecoder(&CurrentCompressedParams, &CurrentUncompressedParams, &DecodeOperationParams, Handle);
			if (Result == DNX_NO_ERROR)
			{
				NumOutputBits = GetNumberOfOutputColorBitsPerPixel(CurrentUncompressedParams.compType);
			}
			else
			{
				DNX_DeleteDecoder(Handle);
				Handle = nullptr;
				NumOutputBits = 0;
				bHasAlpha = false;
			}
		}
	}
	// If we do not have a decoder, or the above reconfiguration failed, create a new one.
	if (!Handle)
	{
		CurrentCompressedParams = CompressedParams;
		CurrentSignalStandard = SignalStandard;
		bHasAlpha = !!CurrentCompressedParams.alphaPresence;

		FMemory::Memzero(DecodeOperationParams);
		DecodeOperationParams.structSize = sizeof(DecodeOperationParams);
		DecodeOperationParams.numThreads = InNumDecodeThreads;
		DecodeOperationParams.decodeResolution = DNX_DecodeResolution_t::DNX_DR_Full;
		DecodeOperationParams.verifyCRC = 0;
		DecodeOperationParams.decodeAlpha = bIgnoreAlpha ? 0 : (bHasAlpha ? 1 : 0);

		if (!SetupDecompressStruct())
		{
			return DNX_INVALID_UNCOMPINFO_ERROR;
		}
		Result = DNX_CreateDecoder(&CurrentCompressedParams, &CurrentUncompressedParams, &DecodeOperationParams, &Handle);
		if (Result != DNX_NO_ERROR)
		{
			UE_LOGF(LogAvidDNxHDElectraDecoder, Log, "DNX_CreateDecoder() failed with %d", Result);
			return Result;
		}

		NumOutputBits = GetNumberOfOutputColorBitsPerPixel(CurrentUncompressedParams.compType);
	}

	return Result;
}

bool FVideoDecoderAvidDNxHDElectra::FDecoderHandle::SetupDecompressStruct()
{
	FMemory::Memzero(CurrentUncompressedParams);
	CurrentUncompressedParams.structSize = sizeof(CurrentUncompressedParams);
	CurrentUncompressedParams.colorVolume = CurrentCompressedParams.colorVolume;
	CurrentUncompressedParams.colorFormat = CurrentCompressedParams.colorFormat;
	CurrentUncompressedParams.fieldOrder = DNX_BufferFieldOrder_t::DNX_BFO_Progressive;
	CurrentUncompressedParams.rgt = DNX_RasterGeometryType_t::DNX_RGT_Display;

	if (CurrentCompressedParams.colorFormat == DNX_ColorFormat_t::DNX_CF_YCbCr)
	{
		switch(CurrentCompressedParams.depth)
		{
			case 8:
			{
				CurrentUncompressedParams.compType = DNX_ComponentType_t::DNX_CT_UCHAR;
				// FIXME: What output format would the alpha channel require us to use?
				if (DecodeOperationParams.decodeAlpha)
				{
					UE_LOGF(LogAvidDNxHDElectraDecoder, Log, "Compressed alpha with YCbYCr is not currently handled");
					return false;
				}
				CurrentUncompressedParams.compOrder = DecodeOperationParams.decodeAlpha ? DNX_ColorComponentOrder_t::DNX_CCO_CbYCrY_A : DNX_ColorComponentOrder_t::DNX_CCO_YCbYCr_NoA;
				break;
			}
			case 12:
			{
				CurrentUncompressedParams.compType = DNX_ComponentType_t::DNX_CT_USHORT_12_4;
				// FIXME: What output format would the alpha channel require us to use?
				if (DecodeOperationParams.decodeAlpha)
				{
					UE_LOGF(LogAvidDNxHDElectraDecoder, Log, "Compressed alpha with YCbYCr is not currently handled");
					return false;
				}
				CurrentUncompressedParams.compOrder = DecodeOperationParams.decodeAlpha ? DNX_ColorComponentOrder_t::DNX_CCO_CbYCrY_A : DNX_ColorComponentOrder_t::DNX_CCO_YCbYCr_NoA;
				break;
			}
			default:
			{
				UE_LOGF(LogAvidDNxHDElectraDecoder, Log, "Compressed %d bit depth is not currently handled", CurrentCompressedParams.depth);
				return false;
			}
		}
	}
	else
	{
		UE_LOGF(LogAvidDNxHDElectraDecoder, Log, "Compressed RGB format is not currently handled");
		return false;
	}

	/*if (CurrentUncompressedParams.compType == DNX_ComponentType_t::DNX_CT_SHORT_2_14)
	{
		CurrentUncompressedParams.blackPoint = DNX_DEFAULT_SHORT_2_14_BLACK_POINT;
		CurrentUncompressedParams.whitePoint = DNX_DEFAULT_SHORT_2_14_WHITE_POINT;
		CurrentUncompressedParams.chromaExcursion = DNX_DEFAULT_CHROMA_SHORT_2_14_EXCURSION;
	}*/
	return true;
}

int32 FVideoDecoderAvidDNxHDElectra::FDecoderHandle::Decode(uint8* OutBufferData, uint32 OutBufferSize, const void* InInput, uint32 InInputSize)
{
	if (!Handle)
	{
		return DNX_NOT_INITIALIZED_ERROR;
	}
	int32 Result = DNX_DecodeFrame(Handle, InInput, OutBufferData, InInputSize, (unsigned int)OutBufferSize);
	return Result;
}


void FVideoDecoderAvidDNxHDElectra::FDecoderHandle::Reset()
{
	if (Handle)
	{
		DNX_DeleteDecoder(Handle);
		Handle = nullptr;
	}
	NumOutputBits = 0;
}

int32 FVideoDecoderAvidDNxHDElectra::FDecoderHandle::GetNumberOfOutputColorBitsPerPixel(DNX_ComponentType_t InOutputComponentType)
{
	switch (InOutputComponentType)
	{
		default:
		case DNX_ComponentType_t::DNX_CT_UCHAR:			// 8 bit
			return 8;
		case DNX_ComponentType_t::DNX_CT_USHORT_10_6:	// 10 bit
		case DNX_ComponentType_t::DNX_CT_10Bit_2_8:		// 10 bit in 2_8 format. Byte ordering is fixed. This is to be used with 10-bit 4:2:2 YCbCr components.
		case DNX_ComponentType_t::DNX_CT_V210:			// Apple's V210
			return 10;
		case DNX_ComponentType_t::DNX_CT_SHORT_2_14:	// Fixed point
		case DNX_ComponentType_t::DNX_CT_SHORT:			// 16 bit. Premultiplied by 257. Byte ordering is machine dependent.
			return 16;
		case DNX_ComponentType_t::DNX_CT_USHORT_12_4:	// 12 bit
			return 12;
	}
}

