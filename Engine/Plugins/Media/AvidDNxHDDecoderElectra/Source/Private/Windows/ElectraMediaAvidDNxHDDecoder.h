// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FElectraMediaAvidDNxHDDecoder
{
public:
	static void Startup();
	static void Shutdown();
};
